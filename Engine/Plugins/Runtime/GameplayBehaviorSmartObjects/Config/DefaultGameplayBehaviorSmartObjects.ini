[CoreRedirects]
+ClassRedirects=(OldName="/Script/SmartObjectsModule.SmartObjectGameplayBehaviorDefinition", NewName="/Script/GameplayBehaviorSmartObjectsModule.GameplayBehaviorSmartObjectBehaviorDefinition")
+ClassRedirects=(OldName="/Script/SmartObjectsModule.AITask_UseSmartObject", NewName="/Script/GameplayBehaviorSmartObjectsModule.AITask_UseGameplayBehaviorSmartObject")
+ClassRedirects=(OldName="/Script/SmartObjectsModule.BTTask_FindAndUseSmartObject", NewName="/Script/GameplayBehaviorSmartObjectsModule.BTTask_FindAndUseGameplayBehaviorSmartObject")
+FunctionRedirects=(OldName="K2_UseSmartObject", NewName="/Script/GameplayBehaviorSmartObjectsModule.GameplayBehaviorSmartObjectsBlueprintFunctionLibrary.UseGameplayBehaviorSmartObject")
+FunctionRedirects=(OldName="AITask_UseGameplayBehaviorSmartObject.UseSmartObject", NewName="AITask_UseGameplayBehaviorSmartObject.UseGameplayBehaviorSmartObject")
+FunctionRedirects=(OldName="AITask_UseGameplayBehaviorSmartObject.UseClaimedSmartObject", NewName="AITask_UseGameplayBehaviorSmartObject.UseClaimedGameplayBehaviorSmartObject")
+PropertyRedirects=(OldName="/Script/GameplayBehaviorSmartObjects.AITask_UseGameplayBehaviorSmartObject.OnFinished",NewName="OnSucceeded")

+FunctionRedirects=(OldName="AITask_UseGameplayBehaviorSmartObject.UseClaimedGameplayBehaviorSmartObject",NewName="AITask_UseGameplayBehaviorSmartObject.MoveToAndUseSmartObjectWithGameplayBehavior")
