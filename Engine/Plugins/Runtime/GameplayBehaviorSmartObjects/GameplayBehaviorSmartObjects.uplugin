{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "GameplayBehaviorSmartObjects",
	"Description": "Plugins for SmartObjects using GameplayBehavior as their default runtime behavior",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "GameplayBehaviorSmartObjectsModule",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "GameplayBehaviors",
			"Enabled": true
		},
		{
			"Name": "SmartObjects",
			"Enabled": true
		}
	]
}