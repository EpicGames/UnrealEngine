// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GameplayBehaviorSmartObjectsModule : ModuleRules
{
	public GameplayBehaviorSmartObjectsModule(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "GameplayBhvSO";
		
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AIModule",
				"Core",
				"GameplayBehaviorsModule",
				"GameplayTags",
				"GameplayTasks",
				"SmartObjectsModule"
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
			}
			);
	}
}
