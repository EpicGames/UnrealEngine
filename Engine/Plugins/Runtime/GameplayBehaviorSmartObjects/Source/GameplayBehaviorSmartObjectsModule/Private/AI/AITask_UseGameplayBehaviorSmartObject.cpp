// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/AITask_UseGameplayBehaviorSmartObject.h"
#include "AIController.h"
#include "DrawDebugHelpers.h"
#include "Engine/World.h"
#include "GameplayBehavior.h"
#include "GameplayBehaviorConfig.h"
#include "GameplayBehaviorSmartObjectBehaviorDefinition.h"
#include "GameplayBehaviorSubsystem.h"
#include "GameplayTagAssetInterface.h"
#include "Misc/ScopeExit.h"
#include "SmartObjectComponent.h"
#include "SmartObjectRequestTypes.h"
#include "SmartObjectSubsystem.h"
#include "Tasks/AITask_MoveTo.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AITask_UseGameplayBehaviorSmartObject)

//----------------------------------------------------------------------//
// Static Blueprint callable and helper functions
//----------------------------------------------------------------------//
UAITask_UseGameplayBehaviorSmartObject* UAITask_UseGameplayBehaviorSmartObject::UseSmartObjectWithGameplayBehavior(AAIController* Controller, const FSmartObjectClaimHandle ClaimHandle, const bool bLockAILogic, ESmartObjectClaimPriority ClaimPriority)
{
	if (Controller == nullptr)
	{
		UE_LOGF(LogSmartObject, Error, "AI Controller required to use smart object.");
		return nullptr;
	}

	const AActor* Pawn = Controller->GetPawn();
	if (Pawn == nullptr)
	{
		UE_LOGF(LogSmartObject, Error, "Pawn required on controller: %ls.", *Controller->GetName());
		return nullptr;
	}

	return UseClaimedSmartObject(*Controller, ClaimHandle, bLockAILogic);
}

UAITask_UseGameplayBehaviorSmartObject* UAITask_UseGameplayBehaviorSmartObject::MoveToAndUseSmartObjectWithGameplayBehavior(AAIController* Controller, const FSmartObjectClaimHandle ClaimHandle, const bool bLockAILogic, ESmartObjectClaimPriority ClaimPriority)
{
	UAITask_UseGameplayBehaviorSmartObject* NewTask = UseSmartObjectWithGameplayBehavior(Controller, ClaimHandle, bLockAILogic, ClaimPriority);
	if (NewTask != nullptr)
	{
		NewTask->SetShouldReachSlotLocation(true);
	}

	return NewTask;
}
UAITask_UseGameplayBehaviorSmartObject* UAITask_UseGameplayBehaviorSmartObject::UseGameplayBehaviorSmartObject(AAIController* Controller, AActor* SmartObjectActor, USmartObjectComponent* SmartObjectComponent, const bool bLockAILogic)
{
	if (SmartObjectComponent == nullptr && SmartObjectActor != nullptr)
	{
		SmartObjectComponent = Cast<USmartObjectComponent>(SmartObjectActor->GetComponentByClass(USmartObjectComponent::StaticClass()));
	}

	return (SmartObjectComponent && Controller)
		? UseSmartObjectComponent(*Controller, *SmartObjectComponent, bLockAILogic, ESmartObjectClaimPriority::Normal)
		: nullptr;
}

UAITask_UseGameplayBehaviorSmartObject* UAITask_UseGameplayBehaviorSmartObject::UseSmartObjectComponent(AAIController& Controller, const USmartObjectComponent& SmartObjectComponent, const bool bLockAILogic, ESmartObjectClaimPriority ClaimPriority)
{
	AActor* Pawn = Controller.GetPawn();
	if (Pawn == nullptr)
	{
		UE_LOGF(LogSmartObject, Error, "Pawn required on controller: %ls.", *Controller.GetName());
		return nullptr;
	}

	USmartObjectSubsystem* SmartObjectSubsystem = USmartObjectSubsystem::GetCurrent(Pawn->GetWorld());
	if (SmartObjectSubsystem == nullptr)
	{
		UE_LOGF(LogSmartObject, Error, "No SmartObjectSubsystem in world: %ls.", *GetNameSafe(Pawn->GetWorld()));
		return nullptr;
	}

	FSmartObjectRequestFilter Filter;
	Filter.ClaimPriority = ClaimPriority;
	Filter.BehaviorDefinitionClasses = { UGameplayBehaviorSmartObjectBehaviorDefinition::StaticClass() };
	const IGameplayTagAssetInterface* TagsSource = Cast<const IGameplayTagAssetInterface>(Pawn);
	if (TagsSource != nullptr)
	{
		TagsSource->GetOwnedGameplayTags(Filter.UserTags);
	}

	TArray<FSmartObjectSlotHandle> SlotHandles;
	const FSmartObjectActorUserData ActorUserData(Pawn);
	const FConstStructView ActorUserDataView(FConstStructView::Make(ActorUserData));

	SmartObjectSubsystem->FindSlots(SmartObjectComponent.GetRegisteredHandle(), Filter, SlotHandles, ActorUserDataView);

	if (SlotHandles.IsEmpty())
	{
		return nullptr;
	}

	const FSmartObjectClaimHandle ClaimHandle = SmartObjectSubsystem->MarkSlotAsClaimed(SlotHandles.Top(), ClaimPriority, ActorUserDataView);

	return ClaimHandle.IsValid() ? UseClaimedSmartObject(Controller, ClaimHandle, bLockAILogic) : nullptr;
}

UAITask_UseGameplayBehaviorSmartObject* UAITask_UseGameplayBehaviorSmartObject::UseClaimedSmartObject(AAIController& Controller, const FSmartObjectClaimHandle ClaimHandle, const bool bLockAILogic)
{
	UAITask_UseGameplayBehaviorSmartObject* MyTask = NewAITask<UAITask_UseGameplayBehaviorSmartObject>(Controller, EAITaskPriority::High);
	if (MyTask == nullptr)
	{
		return nullptr;
	}

	MyTask->SetClaimHandle(ClaimHandle);

	if (bLockAILogic)
	{
		MyTask->RequestAILogicLocking();
	}

	return MyTask;
}

//----------------------------------------------------------------------//
// UAITask_UseGameplayBehaviorSmartObject
//----------------------------------------------------------------------//
UAITask_UseGameplayBehaviorSmartObject::UAITask_UseGameplayBehaviorSmartObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	//bIsPausable = false;
	bBehaviorFinished = false;

	// Uncomment for debug draw while moving toward SO location
	// bTickingTask = true;
}

void UAITask_UseGameplayBehaviorSmartObject::TickTask(const float DeltaTime)
{
	Super::TickTask(DeltaTime);

#if ENABLE_DRAW_DEBUG
	if (MoveToTask != nullptr)
	{
		DrawDebugDirectionalArrow(OwnerController->GetWorld(), OwnerController->GetPawn()->GetActorLocation(), MoveToTask->GetMoveRequestRef().GetGoalLocation(), 10000.f, FColor::Yellow);
	}
#endif // ENABLE_DRAW_DEBUG
}

void UAITask_UseGameplayBehaviorSmartObject::Activate()
{
	Super::Activate();

	bool bSuccess = false;
	ON_SCOPE_EXIT
	{
		if (!bSuccess)
		{
			EndTask();
		}
	};

	if (!ensureMsgf(ClaimedHandle.IsValid(), TEXT("SmartObject handle must be valid at this point.")))
	{
		return;
	}

	if (OwnerController->GetPawn() == nullptr)
	{
		UE_VLOG(OwnerController, LogSmartObject, Error, TEXT("Pawn required to use GameplayBehavior with claim handle: %s."), *LexToString(ClaimedHandle));
		return;
	}

	USmartObjectSubsystem* SmartObjectSubsystem = USmartObjectSubsystem::GetCurrent(OwnerController->GetWorld());
	if (!ensureMsgf(SmartObjectSubsystem != nullptr, TEXT("SmartObjectSubsystem must be accessible at this point.")))
	{
		return;
	}

	// A valid claimed handle can point to an object that is no longer part of the simulation
	if (!SmartObjectSubsystem->IsClaimedSmartObjectValid(ClaimedHandle))
	{
		UE_VLOG(OwnerController, LogSmartObject, Log, TEXT("Claim handle: %s refers to an object that is no longer available."), *LexToString(ClaimedHandle));
		return;
	}

	// Register a callback to be notified if the claimed slot became unavailable
	SmartObjectSubsystem->RegisterSlotInvalidationCallback(ClaimedHandle, FOnSlotInvalidated::CreateUObject(this, &UAITask_UseGameplayBehaviorSmartObject::OnSlotInvalidated));

	if (bShouldUseMoveTo)
	{
		const TOptional<FVector> GoalLocation = SmartObjectSubsystem->GetSlotLocation(ClaimedHandle);
		if (!ensureMsgf(GoalLocation.IsSet(), TEXT("Unable to extract a valid slot location.")))
		{
			return;
		}

		FAIMoveRequest MoveReq(GoalLocation.GetValue());
		MoveReq.SetUsePathfinding(true);
		MoveReq.SetAllowPartialPath(false);

		MoveToTask = NewAITask<UAITask_MoveTo>(*OwnerController, *this, EAITaskPriority::High, TEXT("SmartObject"));
		MoveToTask->SetUp(OwnerController, MoveReq);
		MoveToTask->ReadyForActivation();
		bSuccess = true;
	}
	else
	{
		bSuccess = StartInteraction();
	}
}

bool UAITask_UseGameplayBehaviorSmartObject::StartInteraction()
{
	check(OwnerController);
	UWorld* World = OwnerController->GetWorld();
	USmartObjectSubsystem* SmartObjectSubsystem = USmartObjectSubsystem::GetCurrent(World);
	if (!ensure(SmartObjectSubsystem))
	{
		return false;
	}

	const UGameplayBehaviorSmartObjectBehaviorDefinition* SmartObjectGameplayBehaviorDefinition = SmartObjectSubsystem->MarkSlotAsOccupied<UGameplayBehaviorSmartObjectBehaviorDefinition>(ClaimedHandle);
	const UGameplayBehaviorConfig* GameplayBehaviorConfig = SmartObjectGameplayBehaviorDefinition != nullptr ? SmartObjectGameplayBehaviorDefinition->GameplayBehaviorConfig.Get() : nullptr;
	GameplayBehavior = GameplayBehaviorConfig != nullptr ? GameplayBehaviorConfig->GetBehavior(*World) : nullptr;
	if (GameplayBehavior == nullptr)
	{
		return false;
	}

	const USmartObjectComponent* SmartObjectComponent = SmartObjectSubsystem->GetSmartObjectComponent(ClaimedHandle);
	AActor& InteractorActor = *OwnerController->GetPawn();
	AActor* InteracteeActor = SmartObjectComponent ? SmartObjectComponent->GetOwner() : nullptr;
	const bool bBehaviorActive = UGameplayBehaviorSubsystem::TriggerBehavior(*GameplayBehavior, InteractorActor, GameplayBehaviorConfig, InteracteeActor);
	// Behavior can be successfully triggered AND ended synchronously. We are only interested to register callback when still running
	if (bBehaviorActive)
	{
		OnBehaviorFinishedNotifyHandle = GameplayBehavior->GetOnBehaviorFinishedDelegate().AddUObject(this, &UAITask_UseGameplayBehaviorSmartObject::OnSmartObjectBehaviorFinished);
	}

	return bBehaviorActive;
}

void UAITask_UseGameplayBehaviorSmartObject::OnGameplayTaskDeactivated(UGameplayTask& Task)
{
	if (MoveToTask == &Task)
	{
		if (MoveToTask->IsFinished())
		{
			if (MoveToTask->WasMoveSuccessful())
			{
				MoveToTask = nullptr;
				if (StartInteraction() == false)
				{
					EndTask();	
				}
			}
			else
			{
				OnMoveToFailed.Broadcast();
				EndTask();
			}
		}
	}

	Super::OnGameplayTaskDeactivated(Task);
}

void UAITask_UseGameplayBehaviorSmartObject::OnDestroy(const bool bInOwnerFinished)
{
	if (ClaimedHandle.IsValid())
	{
		if (ensureMsgf(IsValid(OwnerController), TEXT("OwnerController is expected to always be valid")))
		{
			USmartObjectSubsystem* SmartObjectSubsystem = USmartObjectSubsystem::GetCurrent(OwnerController->GetWorld());
			if (IsValid(SmartObjectSubsystem))
			{
				SmartObjectSubsystem->MarkSlotAsFree(ClaimedHandle);
				SmartObjectSubsystem->UnregisterSlotInvalidationCallback(ClaimedHandle);
			}
		}
		ClaimedHandle.Invalidate();
	}

	if (TaskState != EGameplayTaskState::Finished)
	{
		if (GameplayBehavior != nullptr && bBehaviorFinished)
		{
			OnSucceeded.Broadcast();
		}
		else
		{
			OnFailed.Broadcast();
		}
	}

	Super::OnDestroy(bInOwnerFinished);
}

void UAITask_UseGameplayBehaviorSmartObject::OnSlotInvalidated(const FSmartObjectClaimHandle& ClaimHandle, ESmartObjectSlotState State)
{
	Abort();
}

void UAITask_UseGameplayBehaviorSmartObject::Abort()
{
	if (MoveToTask)
	{
		// clear before triggering 'the end' so that OnGameplayTaskDeactivated
		// ignores the incoming info about task end
		UAITask_MoveTo* Task = MoveToTask;
		MoveToTask = nullptr;
		Task->ExternalCancel();
	}
	else if (!bBehaviorFinished)
	{
		if (GameplayBehavior != nullptr)
		{
			check(OwnerController);
			GameplayBehavior->GetOnBehaviorFinishedDelegate().Remove(OnBehaviorFinishedNotifyHandle);

			// Check for a valid pawn first - this is used by the 'shutdown' path so pawn is not guaranteed to still be valid.
			if (APawn* OwnerPawn = OwnerController->GetPawn())
			{
				GameplayBehavior->AbortBehavior(*OwnerPawn);
			}
		}
	}

	EndTask();
}

void UAITask_UseGameplayBehaviorSmartObject::OnSmartObjectBehaviorFinished(UGameplayBehavior& Behavior, AActor& Avatar, const bool bInterrupted)
{
	ensureMsgf(IsValid(OwnerController), TEXT("OwnerController is expected to always be valid"));

	// make sure we handle the right pawn - we can get this notify for a different
	// Avatar if the behavior sending it out is not instanced (CDO is being used to perform actions)
	if (OwnerController && OwnerController->GetPawn() == &Avatar)
	{
		Behavior.GetOnBehaviorFinishedDelegate().Remove(OnBehaviorFinishedNotifyHandle);
		bBehaviorFinished = true;
		EndTask();
	}
}

