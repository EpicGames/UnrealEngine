// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/BTTask_FindAndUseGameplayBehaviorSmartObject.h"
#include "AI/AITask_UseGameplayBehaviorSmartObject.h"
#include "AIController.h"
#include "EnvironmentQuery/EnvQueryManager.h"
#include "EnvQueryItemType_SmartObject.h"
#include "GameplayBehaviorSmartObjectBehaviorDefinition.h"
#include "GameplayTagAssetInterface.h"
#include "SmartObjectRequestTypes.h"
#include "SmartObjectSubsystem.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_FindAndUseGameplayBehaviorSmartObject)

UBTTask_FindAndUseGameplayBehaviorSmartObject::UBTTask_FindAndUseGameplayBehaviorSmartObject()
{
	Radius = 500.f;
	EQSQueryFinishedDelegate = FQueryFinishedSignature::CreateUObject(this, &UBTTask_FindAndUseGameplayBehaviorSmartObject::OnQueryFinished);
	EQSRequest.RunMode = EEnvQueryRunMode::AllMatching;
	bNotifyTaskFinished = true;
}

void UBTTask_FindAndUseGameplayBehaviorSmartObject::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTUseSOTaskMemory>(NodeMemory, InitType);
}

void UBTTask_FindAndUseGameplayBehaviorSmartObject::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTUseSOTaskMemory>(NodeMemory, CleanupType);
}

EBTNodeResult::Type UBTTask_FindAndUseGameplayBehaviorSmartObject::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	EBTNodeResult::Type NodeResult = EBTNodeResult::Failed;

	UWorld* World = GetWorld();
	USmartObjectSubsystem* SmartObjectSubsystem = USmartObjectSubsystem::GetCurrent(World);
	AAIController* MyController = OwnerComp.GetAIOwner();
	if (SmartObjectSubsystem == nullptr || MyController == nullptr
		|| MyController->GetPawn() == nullptr)
	{
		return EBTNodeResult::Failed;
	}

	FBTUseSOTaskMemory* MyMemory = reinterpret_cast<FBTUseSOTaskMemory*>(NodeMemory);
	MyMemory->TaskInstance.Reset();
	MyMemory->EQSRequestID = INDEX_NONE;

	AActor& Avatar = *MyController->GetPawn();

	if (EQSRequest.IsValid() && (EQSRequest.EQSQueryBlackboardKey.IsSet() || EQSRequest.QueryTemplate))
	{
		const UBlackboardComponent* BlackboardComponent = OwnerComp.GetBlackboardComponent();
		MyMemory->EQSRequestID = EQSRequest.Execute(Avatar, BlackboardComponent, EQSQueryFinishedDelegate);

		if (MyMemory->EQSRequestID != INDEX_NONE)
		{
			NodeResult = EBTNodeResult::InProgress;
		}
	}
	else 
	{
		const FVector UserLocation = Avatar.GetActorLocation();

		// Create filter
		FSmartObjectRequestFilter Filter;
		Filter.ActivityRequirements = ActivityRequirements;
		Filter.BehaviorDefinitionClasses = { UGameplayBehaviorSmartObjectBehaviorDefinition::StaticClass() };
		const IGameplayTagAssetInterface* TagsSource = Cast<const IGameplayTagAssetInterface>(&Avatar);
		if (TagsSource != nullptr)
		{
			TagsSource->GetOwnedGameplayTags(Filter.UserTags);
		}

		// Create request
		FSmartObjectRequest Request(FBox(UserLocation, UserLocation).ExpandBy(FVector(Radius), FVector(Radius)), Filter);
		TArray<FSmartObjectRequestResult> Results;
		const FSmartObjectActorUserData ActorUserData(&Avatar);
		const FConstStructView ActorUserDataView(FConstStructView::Make(ActorUserData));
	
		if (SmartObjectSubsystem->FindSmartObjects(Request, Results, ActorUserDataView))
		{
			for (const FSmartObjectRequestResult& Result : Results)
			{
				FSmartObjectClaimHandle ClaimHandle = SmartObjectSubsystem->MarkSlotAsClaimed(Result.SlotHandle, ClaimPriority, ActorUserDataView);
				if (ClaimHandle.IsValid())
				{
					UseClaimedSmartObject(OwnerComp, ClaimHandle, *MyMemory);

					NodeResult = EBTNodeResult::InProgress;
					UE_VLOG_UELOG(MyController, LogSmartObject, Log, TEXT("%s claimed smart object: %s"), *GetNodeName(), *LexToString(ClaimHandle));
					break;
				}
			}

			UE_CVLOG_UELOG(NodeResult == EBTNodeResult::Failed, MyController, LogSmartObject, Warning, TEXT("%s failed to claim smart object"), *GetNodeName());
		}
		else
		{
			UE_VLOG_UELOG(MyController, LogSmartObject
				, Verbose, TEXT("%s failed to find smart objects for request: %s")
				, *GetNodeName(), *Avatar.GetName());
		}
	}

	return NodeResult;
}

EBTNodeResult::Type UBTTask_FindAndUseGameplayBehaviorSmartObject::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	check(NodeMemory);

	FBTUseSOTaskMemory* MyMemory = reinterpret_cast<FBTUseSOTaskMemory*>(NodeMemory);
	if (UAITask_UseGameplayBehaviorSmartObject* UseSOTask = MyMemory->TaskInstance.Get())
	{
		UseSOTask->ExternalCancel();
		MyMemory->TaskInstance.Reset();
	}

	if (MyMemory->EQSRequestID != INDEX_NONE)
	{
		// Clear before calling AbortQuery since it will call us back through OnTaskFinished
		// which uses a check to make sure that the request ID was cleared before reaching that point.
		const int32 EQSRequestID = MyMemory->EQSRequestID;
		MyMemory->EQSRequestID = INDEX_NONE;

		if (UWorld* World = OwnerComp.GetWorld())
		{
			if (UEnvQueryManager* EnvQueryManager = UEnvQueryManager::GetCurrent(World))
			{
				EnvQueryManager->AbortQuery(EQSRequestID);
			}
		}
	}

	return EBTNodeResult::Aborted;
}

void UBTTask_FindAndUseGameplayBehaviorSmartObject::OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult)
{
	FBTUseSOTaskMemory* MyMemory = reinterpret_cast<FBTUseSOTaskMemory*>(NodeMemory);
	if (UAITask_UseGameplayBehaviorSmartObject* UseSOTask = MyMemory->TaskInstance.Get())
	{
		check(UseSOTask->IsFinished());
	}
	check(MyMemory->EQSRequestID == INDEX_NONE);
}

FString UBTTask_FindAndUseGameplayBehaviorSmartObject::GetStaticDescription() const
{
	FString Result;
	if (ActivityRequirements.IsEmpty() == false)
	{
		Result += FString::Printf(TEXT("Object requirements: %s")
			, *ActivityRequirements.GetDescription());
	}

	return Result.Len() > 0
		? Result
		: Super::GetStaticDescription();
}

void UBTTask_FindAndUseGameplayBehaviorSmartObject::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);
	EQSRequest.InitForOwnerAndBlackboard(*this, GetBlackboardAsset());
}

void UBTTask_FindAndUseGameplayBehaviorSmartObject::OnQueryFinished(TSharedPtr<FEnvQueryResult> Result)
{
	if (!Result)
	{
		return;
	}

	AActor* MyOwner = Cast<AActor>(Result->Owner.Get());
	if (APawn* PawnOwner = Cast<APawn>(MyOwner))
	{
		MyOwner = PawnOwner->GetController();
	}

	UBehaviorTreeComponent* BTComponent = MyOwner ? MyOwner->FindComponentByClass<UBehaviorTreeComponent>() : NULL;
	if (!BTComponent)
	{
		UE_LOGF(LogBehaviorTree, Warning, "%ls [%ls]: Unable to find behavior tree to notify about finished query!"
			, ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(MyOwner));
		return;
	}

	const FEnvQueryResult& QueryResult = *Result.Get();

	uint8* RawMemory = BTComponent->GetNodeMemory(this, BTComponent->FindInstanceContainingNode(this));
	check(RawMemory);
	FBTUseSOTaskMemory* MyMemory = reinterpret_cast<FBTUseSOTaskMemory*>(RawMemory);
	if (MyMemory->EQSRequestID != QueryResult.QueryID)
	{
		UE_VLOG_UELOG(BTComponent, LogBehaviorTree, Log, TEXT("%s [%s] ignoring EQS result due to QueryID mismatch.")
			, ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(MyOwner));

		check(MyMemory->EQSRequestID != INDEX_NONE)

		return;
	}
	else if (QueryResult.IsAborted())
	{
		UE_VLOG_UELOG(BTComponent, LogBehaviorTree, Log, TEXT("%s [%s] observed EQS query finished as Aborted. Aborting the BT node as well.")
			, ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(MyOwner));

		FinishLatentTask(*BTComponent, EBTNodeResult::Aborted);
		return;
	}

	// at this point we've already confirmed that QueryResult does indeed corresponds to the the query we're waiting for
	// so we need to clear the EQSRequestID here in case the next task we're about to issue (the UAITask_UseGameplayBehaviorSmartObject)
	// might fail instantly and we do check EQSRequestID on tasks end to make sure everything has been cleaned up properly.
	MyMemory->EQSRequestID = INDEX_NONE;

	bool bSmartObjectClaimed = false;

	if (QueryResult.IsSuccessful() && (QueryResult.Items.Num() >= 1))
	{
		if (QueryResult.ItemType->IsChildOf(UEnvQueryItemType_SmartObject::StaticClass()) == false)
		{
			UE_VLOG_UELOG(BTComponent, LogSmartObject, Error, TEXT("%s used EQS query that did not generate EnvQueryItemType_SmartObject items"), *GetNodeName());
		}
		else if (USmartObjectSubsystem* SmartObjectSubsystem = USmartObjectSubsystem::GetCurrent(MyOwner->GetWorld()))
		{
			const FSmartObjectActorUserData ActorUserData(Cast<AActor>(Result->Owner.Get()));
			const FConstStructView ActorUserDataView(FConstStructView::Make(ActorUserData));

			// we could use QueryResult.GetItemAsTypeChecked, but the below implementation is more efficient
			for (int i = 0; i < QueryResult.Items.Num(); ++i)
			{
				const FSmartObjectSlotEQSItem& Item = UEnvQueryItemType_SmartObject::GetValue(QueryResult.GetItemRawMemory(i));
				const FSmartObjectClaimHandle ClaimHandle = SmartObjectSubsystem->MarkSlotAsClaimed(Item.SlotHandle, ClaimPriority, ActorUserDataView);
				if (ClaimHandle.IsValid())
				{
					UseClaimedSmartObject(*BTComponent, ClaimHandle, *MyMemory);
					bSmartObjectClaimed = true;

					UE_VLOG_UELOG(BTComponent, LogSmartObject, Log, TEXT("%s claimed EQS-found smart object: %s"), *GetNodeName(), *LexToString(ClaimHandle));
					break;
				}
			}
		}
	}
	
	if (bSmartObjectClaimed == false)
	{
		FinishLatentTask(*BTComponent, EBTNodeResult::Failed);
	}
}

void UBTTask_FindAndUseGameplayBehaviorSmartObject::UseClaimedSmartObject(UBehaviorTreeComponent& OwnerComp, FSmartObjectClaimHandle ClaimHandle, FBTUseSOTaskMemory& MyMemory)
{
	checkSlow(ClaimHandle.IsValid());
	UAITask_UseGameplayBehaviorSmartObject* UseSOTask = NewBTAITask<UAITask_UseGameplayBehaviorSmartObject>(OwnerComp);
	UseSOTask->SetClaimHandle(ClaimHandle);
	UseSOTask->SetShouldReachSlotLocation(true);
	UseSOTask->ReadyForActivation();

	MyMemory.TaskInstance = UseSOTask;
}
