// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayBehaviorSmartObjectsBlueprintFunctionLibrary.h"

#include "AI/AITask_UseGameplayBehaviorSmartObject.h"
#include "Blueprint/AIBlueprintHelperLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayBehaviorSmartObjectsBlueprintFunctionLibrary)

bool UGameplayBehaviorSmartObjectsBlueprintFunctionLibrary::UseGameplayBehaviorSmartObject(AActor* Avatar, AActor* SmartObject)
{
	if (Avatar == nullptr || SmartObject == nullptr)
	{
		return false;
	}

	AAIController* AIController = UAIBlueprintHelperLibrary::GetAIController(Avatar);
	if (AIController != nullptr)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UAITask_UseGameplayBehaviorSmartObject* Task = UAITask_UseGameplayBehaviorSmartObject::UseGameplayBehaviorSmartObject(AIController, SmartObject, nullptr);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (Task != nullptr)
		{
			Task->ReadyForActivation();
		}
		return Task != nullptr;
	}

	return false;
}
