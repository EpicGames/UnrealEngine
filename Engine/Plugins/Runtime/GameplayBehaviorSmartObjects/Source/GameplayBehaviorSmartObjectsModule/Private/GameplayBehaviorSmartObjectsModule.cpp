// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayBehaviorSmartObjectsModule.h"
#include "Modules/ModuleManager.h"


//----------------------------------------------------------------------//
// IGameplayBehaviorSmartObjectsModule
//----------------------------------------------------------------------//
IGameplayBehaviorSmartObjectsModule& IGameplayBehaviorSmartObjectsModule::Get()
{
	return FModuleManager::LoadModuleChecked<IGameplayBehaviorSmartObjectsModule>("GameplayBehaviorSmartObjectsModule");
}

bool IGameplayBehaviorSmartObjectsModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded("GameplayBehaviorSmartObjectsModule");
}


//----------------------------------------------------------------------//
// FGameplayBehaviorSmartObjectsModule
//----------------------------------------------------------------------//
class FGameplayBehaviorSmartObjectsModule : public IGameplayBehaviorSmartObjectsModule
{
};

IMPLEMENT_MODULE(FGameplayBehaviorSmartObjectsModule, GameplayBehaviorSmartObjectsModule)
