// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/AITask.h"
#include "SmartObjectRuntime.h"
#include "AITask_UseGameplayBehaviorSmartObject.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMARTOBJECTSMODULE_API


class AAIController;
class UAITask_MoveTo;
class UGameplayBehavior;
class USmartObjectComponent;

UCLASS(MinimalAPI)
class UAITask_UseGameplayBehaviorSmartObject : public UAITask
{
	GENERATED_BODY()

public:
	UE_API UAITask_UseGameplayBehaviorSmartObject(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/**
	 * Helper function to create an AITask that interacts with the SmartObject slot using the GameplayBehavior definition
	 * This version starts the interaction on spot so the actor needs to be at the desired position. 
	 * @param Controller The controller for which the attached pawn will take part to the GameplayBehavior.
	 * @param ClaimHandle The handle to an already claimed slot.
	 * @param bLockAILogic Indicates if the task adds UAIResource_Logic to the set of Claimed resources 
	 * @return The AITask executing the GameplayBehavior.
	 */
	UFUNCTION(BlueprintCallable, Category = "AI|Tasks", meta = (DefaultToSelf = "Controller" , BlueprintInternalUseOnly = "true"))
	static UE_API UAITask_UseGameplayBehaviorSmartObject* UseSmartObjectWithGameplayBehavior(AAIController* Controller, FSmartObjectClaimHandle ClaimHandle, bool bLockAILogic = true, ESmartObjectClaimPriority ClaimPriority = ESmartObjectClaimPriority::Normal);

	/**
	 * Helper function to create an AITask that reaches and interacts with the SmartObject slot using the GameplayBehavior definition.
	 * @param Controller The controller that will move to the slot location and for which the attached pawn will take part to the GameplayBehavior.
	 * @param ClaimHandle The handle to an already claimed slot.
	 * @param bLockAILogic Indicates if the task adds UAIResource_Logic to the set of Claimed resources 
	 * @return The AITask performing the move to slot location and then executing the GameplayBehavior.
	 */
	UFUNCTION(BlueprintCallable, Category = "AI|Tasks", meta = (DefaultToSelf = "Controller" , BlueprintInternalUseOnly = "true"))
	static UE_API UAITask_UseGameplayBehaviorSmartObject* MoveToAndUseSmartObjectWithGameplayBehavior(AAIController* Controller, FSmartObjectClaimHandle ClaimHandle, bool bLockAILogic = true, ESmartObjectClaimPriority ClaimPriority = ESmartObjectClaimPriority::Normal);

	UE_DEPRECATED(5.3, "Please use one of the UseSmartObjectWithGameplayBehavior functions using claim handle instead")
	UFUNCTION(BlueprintCallable, Category = "AI|Tasks", meta = (DefaultToSelf = "Controller" , BlueprintInternalUseOnly = "true"), meta = (DeprecatedFunction, DeprecationMessage = "Use one of the UseSmartObjectWithGameplayBehavior functions using claim handle instead"))
	static UE_API UAITask_UseGameplayBehaviorSmartObject* UseGameplayBehaviorSmartObject(AAIController* Controller, AActor* SmartObjectActor, USmartObjectComponent* SmartObjectComponent, bool bLockAILogic = true);

	void SetShouldReachSlotLocation(const bool bUseMoveTo) { bShouldUseMoveTo = bUseMoveTo; }
	void SetClaimHandle(const FSmartObjectClaimHandle& Handle) { ClaimedHandle = Handle; }

protected:
	UE_API virtual void Activate() override;
	UE_API virtual void TickTask(float DeltaTime) override;
	UE_API virtual void OnGameplayTaskDeactivated(UGameplayTask& Task) override;
	UE_API virtual void OnDestroy(bool bInOwnerFinished) override;

	UE_API bool StartInteraction();

	UE_API void Abort();

	UE_API void OnSmartObjectBehaviorFinished(UGameplayBehavior& Behavior, AActor& Avatar, const bool bInterrupted);

	UE_API void OnSlotInvalidated(const FSmartObjectClaimHandle& ClaimHandle, const ESmartObjectSlotState State);

	static UE_API UAITask_UseGameplayBehaviorSmartObject* UseSmartObjectComponent(AAIController& Controller, const USmartObjectComponent& SmartObjectComponent, bool bLockAILogic, ESmartObjectClaimPriority ClaimPriority = ESmartObjectClaimPriority::Normal);
	static UE_API UAITask_UseGameplayBehaviorSmartObject* UseClaimedSmartObject(AAIController& Controller, FSmartObjectClaimHandle ClaimHandle, bool bLockAILogic);

protected:
	UPROPERTY(BlueprintAssignable)
	FGenericGameplayTaskDelegate OnSucceeded;

	UPROPERTY(BlueprintAssignable)
	FGenericGameplayTaskDelegate OnFailed;

	UPROPERTY(BlueprintAssignable)
	FGenericGameplayTaskDelegate OnMoveToFailed;

	UPROPERTY()
	TObjectPtr<UAITask_MoveTo> MoveToTask;

	UPROPERTY()
	TObjectPtr<UGameplayBehavior> GameplayBehavior;

	FSmartObjectClaimHandle ClaimedHandle;
	FDelegateHandle OnBehaviorFinishedNotifyHandle;

	bool bBehaviorFinished;

	bool bShouldUseMoveTo;
};

#undef UE_API
