// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SmartObjectTypes.h"
#include "BehaviorTree/BTTaskNode.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "BTTask_FindAndUseGameplayBehaviorSmartObject.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMARTOBJECTSMODULE_API

class UAITask_UseGameplayBehaviorSmartObject;
class AITask_UseSmartObject;
struct FSmartObjectClaimHandle;

struct FBTUseSOTaskMemory
{
	TWeakObjectPtr<UAITask_UseGameplayBehaviorSmartObject> TaskInstance;
	int32 EQSRequestID;
};

/**
*
*/
UCLASS(MinimalAPI)
class UBTTask_FindAndUseGameplayBehaviorSmartObject : public UBTTaskNode
{
	GENERATED_BODY()
public:
	UE_API UBTTask_FindAndUseGameplayBehaviorSmartObject();

protected:
	UE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	UE_API virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	UE_API virtual void OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult) override;
	virtual uint16 GetInstanceMemorySize() const override { return sizeof(FBTUseSOTaskMemory); }
	UE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	UE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;

	UE_API virtual FString GetStaticDescription() const override;

	UE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;

	UE_API void OnQueryFinished(TSharedPtr<FEnvQueryResult> Result);

	UE_API void UseClaimedSmartObject(UBehaviorTreeComponent& OwnerComp, FSmartObjectClaimHandle ClaimHandle, FBTUseSOTaskMemory& MyMemory);

protected:
	/** Additional tag query to filter available smart objects. We'll query for smart
	 *	objects that support activities tagged in a way matching the filter.
	 *	Note that regular tag-base filtering is going to take place as well */
	UPROPERTY(EditAnywhere, Category = SmartObjects)
	FGameplayTagQuery ActivityRequirements;

	UPROPERTY(EditAnywhere, Category = SmartObjects)
	ESmartObjectClaimPriority ClaimPriority = ESmartObjectClaimPriority::Normal;

	UPROPERTY(EditAnywhere, Category = SmartObjects)
	FEQSParametrizedQueryExecutionRequest EQSRequest;

	/** Used for smart object querying if EQSRequest is not configured */
	UPROPERTY(EditAnywhere, Category = SmartObjects, meta=(DisplayName="Fallback Radius"))
	float Radius;

	FQueryFinishedSignature EQSQueryFinishedDelegate; 
};

#undef UE_API
