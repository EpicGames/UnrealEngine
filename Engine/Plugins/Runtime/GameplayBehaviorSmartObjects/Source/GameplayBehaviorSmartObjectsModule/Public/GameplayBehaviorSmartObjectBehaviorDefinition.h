// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SmartObjectDefinition.h"
#include "GameplayBehaviorSmartObjectBehaviorDefinition.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMARTOBJECTSMODULE_API

class UGameplayBehaviorConfig;

/**
 * SmartObject behavior definition for the GameplayBehavior framework
 */
UCLASS(MinimalAPI)
class UGameplayBehaviorSmartObjectBehaviorDefinition : public USmartObjectBehaviorDefinition
{
	GENERATED_BODY()

public:
	UPROPERTY(EditDefaultsOnly, Category = SmartObject, Instanced)
	TObjectPtr<UGameplayBehaviorConfig> GameplayBehaviorConfig;

public:
#if WITH_EDITOR
	//~ UObject
	UE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
#endif //WITH_EDITOR
};

#undef UE_API
