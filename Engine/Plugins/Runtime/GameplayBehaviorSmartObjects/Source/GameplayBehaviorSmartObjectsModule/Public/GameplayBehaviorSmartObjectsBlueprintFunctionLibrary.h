// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "GameplayBehaviorSmartObjectsBlueprintFunctionLibrary.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMARTOBJECTSMODULE_API

class AActor;

UCLASS(MinimalAPI, meta = (ScriptName = "GameplayBehaviorsLibrary"))
class UGameplayBehaviorSmartObjectsBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	UE_DEPRECATED(5.3, "Use MoveToAndUseSmartObjectWithGameplayBehavior or UseSmartObjectWithGameplayBehavior using a claim handle from UAITask_UseGameplayBehaviorSmartObject instead.")
	UFUNCTION(BlueprintCallable, Category = "SmartObject", meta = (DeprecatedFunction, DeprecationMessage = "Use MoveToAndUseSmartObjectWithGameplayBehavior or UseSmartObjectWithGameplayBehavior using a claim handle from UAITask_UseGameplayBehaviorSmartObject instead."))
	static UE_API bool UseGameplayBehaviorSmartObject(AActor* Avatar, AActor* SmartObject);
};

#undef UE_API
